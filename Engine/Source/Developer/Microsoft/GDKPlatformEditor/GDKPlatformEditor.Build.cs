// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GDKPlatformEditor : ModuleRules
{
	public GDKPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"MainFrame",
				"Slate",
				"SlateCore",
				"PropertyEditor",
				"SharedSettingsWidgets",
				"SourceControl",
				"UnrealEd",
				"DeveloperToolSettings",
				"XmlParser",
				"Projects",
				"ToolWidgets",
            }
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
			}
		);
	}
}
