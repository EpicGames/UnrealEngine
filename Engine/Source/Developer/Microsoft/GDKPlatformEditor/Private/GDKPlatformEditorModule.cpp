// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKPlatformEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"
#include "GDKTargetSettings.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"
#include "PropertyEditorModule.h"
#include "GDKTargetSettingsDetails.h"
#include "XmlParser.h"
#include "Misc/MonitoredProcess.h"
#include "Dialog/SMessageDialog.h"
#include "Async/Async.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"


#define LOCTEXT_NAMESPACE "FGDKPlatformEditorModule"

DEFINE_LOG_CATEGORY_STATIC(LogGDKPlatform, Log, All);


/**
* Module for GDK platform editor utilities
*/
class FGDKPlatformEditorModule
	: public IGDKPlatformEditorModule
{
	virtual void StartupModule() override
	{
		// register detail customization
		static FName PropertyEditor("PropertyEditor");
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(PropertyEditor);

		if (PropertyModule != nullptr)
		{
			// customizing the string resources struct so that the image picker dialog button can be included too
			static FName GDKPackageStringResources("GDKPackageStringResources");
			PropertyModule->RegisterCustomPropertyTypeLayout(GDKPackageStringResources, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGDKCultureResourceDetails::MakeInstance));
		}
	}

	virtual void ShutdownModule() override
	{
	}


	void PartnerCenterError( const FText& Message )
	{
		TSharedRef<SMessageDialog> Dialog = SNew(SMessageDialog)
			.Title(LOCTEXT("PartnerCenterErrorTitle","Partner Center"))
			.Message(Message)
			.Buttons({SMessageDialog::FButton(LOCTEXT("OK","OK"))})
		;
		Dialog->ShowModal();
	}

	virtual void PartnerCenterQueryProductsAsync( TFunction<void(bool)> OnComplete )
	{
		FScopeLock Lock(&PartnerCenterToolProcessCS);

		// make sure we're not already running a query
		if (PartnerCenterToolProcess.IsValid())
		{
			OnComplete(false);
			return;
		}

		// make sure the tool exists
		FString PartnerCenterTool = FPaths::Combine( FPaths::EngineDir(), TEXT("Binaries\\DotNET\\Microsoft\\PartnerCenterTool\\PartnerCenterTool.exe"));
		if (!FPaths::FileExists(PartnerCenterTool))
		{
			PartnerCenterError( FText::Format( LOCTEXT("PartnerCenterToolNotFoundFmt","{0} - file not found"), FText::FromString(PartnerCenterTool) ));
			OnComplete(false);
			return;
		}

		// run the tool
		PartnerCenterToolProcess = MakeShared<FMonitoredProcess>(PartnerCenterTool, TEXT("products"), true );
		PartnerCenterToolProcess->OnCompleted().BindLambda( [this, OnComplete = MoveTemp(OnComplete)](int32) mutable
		{
			// defer one tick so we can read the process output
			AsyncTask( ENamedThreads::GameThread, [this, OnComplete = MoveTemp(OnComplete)]() mutable
			{
				bool bResult = ParsePartnerCenterQueryResults();
				OnComplete(bResult);
				PartnerCenterToolProcess.Reset();
			});
		});
		if (!PartnerCenterToolProcess->Launch())
		{
			PartnerCenterError( FText::Format( LOCTEXT("PartnerCenterToolLaunchFail","{0} - launch failed"), FText::FromString(PartnerCenterTool) ));
			OnComplete(false);
			PartnerCenterToolProcess.Reset();
			return;
		}
	}

	bool ParsePartnerCenterQueryResults()
	{
		FScopeLock Lock(&PartnerCenterToolProcessCS);

		if (!PartnerCenterToolProcess.IsValid())
		{
			return false;
		}

		const FString& FullOutput = PartnerCenterToolProcess->GetFullOutputWithoutDelegate();
		if (!FullOutput.IsEmpty())
		{
			// read the xml output & recreate product list
			FXmlFile XmlFile( FullOutput, EConstructMethod::ConstructFromBuffer );
			if (XmlFile.IsValid() && XmlFile.GetRootNode() != nullptr)
			{
				PartnerCenterProducts.Reset();
				for (FXmlNode* ProductNode : XmlFile.GetRootNode()->GetChildrenNodes() )
				{
					TSharedPtr<IGDKPlatformEditorModule::FPartnerCenterProduct> Product = MakeShared<IGDKPlatformEditorModule::FPartnerCenterProduct>();
					PartnerCenterProducts.Add(Product);

					// ensure these two values are always defined to avoid checking for them everywhere they're used
					Product->Add(TEXT("PackageDisplayName"), TEXT("(unnamed product)"));
					Product->Add(TEXT("PackageName"),        TEXT("(invalid package name)"));

					// populate the product dictionary
					for (FXmlNode* DataNode : ProductNode->GetChildrenNodes())
					{
						Product->Add(DataNode->GetTag(), DataNode->GetContent());
					}
				}

				// sort the product list based on display name
				PartnerCenterProducts.Sort( []( const TSharedPtr<FPartnerCenterProduct> A, const TSharedPtr<FPartnerCenterProduct> B )
				{
					const TCHAR* SortKey = TEXT("PackageDisplayName");
					return A->FindRef(SortKey) < B->FindRef(SortKey);
				});

				return true;
			}
		}

		PartnerCenterError( FText::Format( LOCTEXT("PartnerCenterToolFailFmt", "Could not retrieve titles from Partner Center\n{0}"), FText::FromString(FullOutput) ));
		return false;
	}

	virtual const TArray<TSharedPtr<FPartnerCenterProduct>>* GetPartnerCenterProducts() override
	{
		return &PartnerCenterProducts;
	}

	virtual bool IsQueryingPartnerCenter() const override
	{
		FScopeLock Lock(&PartnerCenterToolProcessCS);
		return PartnerCenterToolProcess.IsValid();
	}

	virtual void CancelQueryPartnerCenter() override
	{
		FScopeLock Lock(&PartnerCenterToolProcessCS);

		if (PartnerCenterToolProcess.IsValid())
		{
			PartnerCenterToolProcess->Cancel();
			PartnerCenterToolProcess.Reset();
		}
	}

	TSharedPtr<FMonitoredProcess> PartnerCenterToolProcess;
	TArray<TSharedPtr<FPartnerCenterProduct>> PartnerCenterProducts;
	mutable FCriticalSection PartnerCenterToolProcessCS;
};


IMPLEMENT_MODULE(FGDKPlatformEditorModule, GDKPlatformEditor);

#undef LOCTEXT_NAMESPACE
