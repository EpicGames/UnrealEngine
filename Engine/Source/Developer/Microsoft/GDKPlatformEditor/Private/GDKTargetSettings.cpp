// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKTargetSettings.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Interfaces/IPluginManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GDKTargetSettings)

UGDKTargetSettings::UGDKTargetSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


TArray<FString> UGDKTargetSettings::GetKnownDeviceTypes()
{
	return TArray<FString>();
}

TArray<FString> UGDKTargetSettings::GetKnownIntelligentDeliveryFeatures()
{
	TArray<FString> KnownFeatures;
	for ( const FGDKIntelligentDeliveryFeature& Feature : IntelligentDeliveryFeatures )
	{
		KnownFeatures.AddUnique(Feature.Id);
	}

	KnownFeatures.Sort();
	return KnownFeatures;	
}

TArray<FString> UGDKTargetSettings::GetKnownCultureStageIds()
{
	return GetDefault<UProjectPackagingSettings>()->CulturesToStage;	
}

TArray<FString> UGDKTargetSettings::GetKnownDLCPlugins()
{
	TArray<FString> KnownDLC;

	for (TSharedRef<IPlugin> Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		// note: allowing Disabled plugins here too because typically the DLC plugin needs to be disabled to prevent it from being included in the main game package
		if (Plugin->CanContainContent() && Plugin->GetType() == EPluginType::Project && Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project && Plugin->GetDescriptor().bExplicitlyLoaded && Plugin->GetDescriptor().SupportsTargetPlatform(GetPlatformName()) )
		{
			FString DLCName = Plugin->GetName();
			KnownDLC.Add(DLCName);
		}
	}

	return KnownDLC;
}

TArray<FString> UGDKTargetSettings::GetKnownDLCNames()
{
	TArray<FString> DLCNames;

	for (const FGDKDLCPackage& DLCPackage : DLCPackages)
	{
		DLCNames.Add(DLCPackage.DLCName);
	}

	return DLCNames;
}


bool UGDKTargetSettings::CanEditProductId() const
{
	return StoreId.IsEmpty();
}
