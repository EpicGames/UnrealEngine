// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKTargetSettingsDetails.h"

#include "SExternalImageReference.h"
#include "EditorDirectories.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Views/SListView.h"
#include "Misc/MessageDialog.h"
#include "Misc/NotifyHook.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "HAL/PlatformFileManager.h"
#include "Dialogs/Dialogs.h"
#include "GDKTargetSettings.h"
#include "ObjectEditorUtils.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"
#include "IPropertyUtilities.h"
#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "GDKTargetSettingsDetails"

namespace GDKTargetSettingsDetailsConstants
{
	/** Relative path from engine platform extensions or project platform extensions dir to build resources */
	const FString BuildResourcePath(TEXT("Build/Resources"));
	const FString EngineResourcePath(TEXT("Build/DefaultImages"));
}



FGDKTargetSettingsDetails::FGDKTargetSettingsDetails(const TCHAR* InPlatformName)
	: PlatformName(InPlatformName)
{
}


void FGDKTargetSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& PartnerCenterCategory = DetailBuilder.EditCategory("PartnerCenter", FText::GetEmpty(), ECategoryPriority::Important);
	IDetailCategoryBuilder& RecipesCategory = DetailBuilder.EditCategory("Chunk Install", FText::GetEmpty(), ECategoryPriority::Uncommon);
	const TSharedRef<IPropertyUtilities> PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	// find the target settings that are being customized
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	TWeakObjectPtr<UGDKTargetSettings> GDKTargetSettings = nullptr;
	for (auto Itr : ObjectsBeingCustomized)
	{
		GDKTargetSettings = Cast<UGDKTargetSettings>(Itr.Get());
		if (GDKTargetSettings.IsValid())
		{
			break;
		}
	}


	// add the partner center toolbar
	TSharedPtr<SHorizontalBox> PartnerCenterBox;
	PartnerCenterCategory.AddCustomRow(FText::GetEmpty(), false)
	.WholeRowWidget
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SAssignNew(PartnerCenterBox, SHorizontalBox)
		]
	];

	// if there's nothing set up for this project yet, give the option to configure the game from partner center etc.
	AddGettingStartedItems(GDKTargetSettings, PartnerCenterBox, PropertyUtilities);


	//add a link to Partner Center, optionally including the StoreId for the title
	auto PartnerCenterURL = [GDKTargetSettings]()
	{
		FString Result(TEXT("https://partner.microsoft.com/dashboard/"));
		if (GDKTargetSettings.IsValid() && GDKTargetSettings->StoreId.Len() == 12 )
		{
			Result += FString::Printf(TEXT("products/%s/"), *GDKTargetSettings->StoreId);
		}
		return Result;
	};
	PartnerCenterBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(8,4)
	[
		SNew(SHyperlink)
		.Text(LOCTEXT("PartnerCenterHyperLink", "Open Partner Center"))
		.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
		.ToolTipText_Lambda([=]() { return FText::FromString(*PartnerCenterURL()); })
		.OnNavigate_Lambda([=]() { FPlatformProcess::LaunchURL(*PartnerCenterURL(), nullptr, nullptr); })
	];

	//add a link to the stores, if the store id has been set
	auto XboxStoreURL = [GDKTargetSettings]()
	{
		return GDKTargetSettings.IsValid() ? FString::Printf(TEXT("msxbox://game/?productid=%s"), *GDKTargetSettings->StoreId) : TEXT("");
	};
	auto WindowsStoreURL = [GDKTargetSettings]()
	{
		return GDKTargetSettings.IsValid() ? FString::Printf(TEXT("ms-windows-store://pdp/?productid=%s"), *GDKTargetSettings->StoreId) : TEXT("");
	};
	auto HasValidStoreId = [GDKTargetSettings]()
	{
		return (GDKTargetSettings.IsValid() && GDKTargetSettings->StoreId.Len() == 12 ) ? EVisibility::Visible : EVisibility::Collapsed; 
	};
	PartnerCenterBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(8,4)
	[
		SNew(SHyperlink)
		.Text(LOCTEXT("XboxPCStoreHyperLink", "Show In Xbox PC App"))
		.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
		.ToolTipText(LOCTEXT("XboxPCStoreHyperLinkTip", "Open this title in the Xbox PC App. The title must already have been published to the current sandbox."))
		.OnNavigate_Lambda([=]() { FPlatformProcess::LaunchURL(*XboxStoreURL(), nullptr, nullptr); })
		.Visibility(TAttribute<EVisibility>::Create(HasValidStoreId))
	];
	PartnerCenterBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(8,4)
	[
		SNew(SHyperlink)
		.Text(LOCTEXT("WindowsStoreHyperLink", "Show In Windows Store"))
		.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
		.ToolTipText(LOCTEXT("WindowsStoreHyperLinkTip", "Open this title in the Windows Store. The title must already have been published to the current sandbox."))
		.OnNavigate_Lambda([=]() { FPlatformProcess::LaunchURL(*WindowsStoreURL(), nullptr, nullptr); })
		.Visibility(TAttribute<EVisibility>::Create(HasValidStoreId))
	];

	// hook for adding any additional platform-specific items
	AddAdditionalPlatformItems(PartnerCenterBox, GDKTargetSettings);


	//add button to update product details from partner center
	PartnerCenterBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(8,4)
	[
		SNew(SButton)
		.Text(LOCTEXT("UpdateFromPartnerCenterLabel", "Update Partner Center Association"))
		.ToolTipText(LOCTEXT("UpdateFromPartnerCenterTooltip", "Connect to Partner Center to display a list of products, and select one to associate with this game"))
		.ButtonStyle(FAppStyle::Get(), "Button")
		.Visibility_Lambda([this, GDKTargetSettings]() { return !IsGDKConfigured(GDKTargetSettings.Get()) ? EVisibility::Collapsed : EVisibility::Visible; })
		.IsEnabled_Lambda([this]() { return !PartnerCenterProductWindow.IsValid() && !IGDKPlatformEditorModule::Get().IsQueryingPartnerCenter(); } )
		.OnClicked_Lambda([this, GDKTargetSettings, PropertyUtilities]()
		{
			OnClickUpdateFromPartnerCenter(GDKTargetSettings.Get(), PropertyUtilities);
			return FReply::Handled();
		})
	];

	// hide the DLC category if the GDK DLC support plugin isn't enabled
	TSharedPtr<IPlugin> GDKDLCPlugin = IPluginManager::Get().FindPlugin(TEXT("GDKPlatformDLC"));
	if (!GDKDLCPlugin.IsValid() || !GDKDLCPlugin->IsEnabled())
	{
		DetailBuilder.HideCategory("DLC");
	}
}



void FGDKTargetSettingsDetails::AddGettingStartedItems(TWeakObjectPtr<UGDKTargetSettings> GDKTargetSettings, TSharedPtr<SHorizontalBox> PartnerCenterBox, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	TSharedPtr<SHorizontalBox> GettingStartedBox;

	PartnerCenterBox->AddSlot()
	[
		SNew(SBorder)
		.BorderBackgroundColor(FColorList::SteelBlue)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.LightGroupBorder"))
		.Visibility_Lambda([this, GDKTargetSettings]() { return IsGDKConfigured(GDKTargetSettings.Get()) ? EVisibility::Collapsed : EVisibility::Visible; })
		.Padding(8.0f)
		[
			SAssignNew(GettingStartedBox, SHorizontalBox)

			// Status icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("SettingsEditor.WarningIcon"))
			]

			// Notice
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(16.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.ShadowColorAndOpacity(FLinearColor::Black)
				.ShadowOffset(FVector2D::UnitVector)
				.Text(LOCTEXT("NotConfiguredLabel", "This project has not been configured for this platform yet. You can associate your game with Partner Center to get started."))
			]

			// Store Association Button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("ImportFromPartnerCenterLabel", "Associate With Partner Center"))
				.ToolTipText(LOCTEXT("ImportFromPartnerCenterTooltip", "Connect to Partner Center to display a list of products, and select one to associate with this game"))
				.ButtonStyle(FAppStyle::Get(), "Button")
				.IsEnabled_Lambda([this]() { return !PartnerCenterProductWindow.IsValid() && !IGDKPlatformEditorModule::Get().IsQueryingPartnerCenter(); } )
				.OnClicked_Lambda([this, GDKTargetSettings, PropertyUtilities]()
				{
					OnClickUpdateFromPartnerCenter(GDKTargetSettings.Get(), PropertyUtilities);
					return FReply::Handled();
				})
			]
		]
	];

	if (GettingStartedBox.IsValid())
	{
		AddAdditionalGettingStartedItems(GettingStartedBox, GDKTargetSettings, PropertyUtilities);
	}
}


bool FGDKTargetSettingsDetails::IsGDKConfigured(UGDKTargetSettings* GDKTargetSettings) const
{
	return (GDKTargetSettings == nullptr || GDKTargetSettings->TitleId.Len() > 0 || GDKTargetSettings->PrimaryServiceConfigId.Len() > 0 || GDKTargetSettings->StoreId.Len() > 0 || GDKTargetSettings->PackageName.Len() > 0);
}

void FGDKTargetSettingsDetails::NotifyPropertyChanged(FProperty* Property, UObject* OwnerObject, const TSharedRef<class IPropertyUtilities> PropertyUtilities) const
{
	// signal that the property has changed - this makes sure the ini file is updated & saved
	FNotifyHook* NotifyHook = PropertyUtilities->GetNotifyHook();
	if (NotifyHook != nullptr)
	{
		TArray<const UObject*> NotifyTopLevelObjects;
		NotifyTopLevelObjects.Add(OwnerObject);

		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(Property);

		FPropertyChangedEvent ChangeEvent(Property, EPropertyChangeType::ValueSet, MakeArrayView(NotifyTopLevelObjects));
		NotifyHook->NotifyPostChange(ChangeEvent, &PropertyChain);
	}
}

void FGDKTargetSettingsDetails::NotifyPropertyChanged(FName PropertyName, UObject* OwnerObject, const TSharedRef<class IPropertyUtilities> PropertyUtilities) const
{
	FProperty* Property = FindFProperty<FProperty>( OwnerObject->GetClass(), PropertyName );
	if (Property != nullptr)
	{
		NotifyPropertyChanged(Property, OwnerObject, PropertyUtilities);
	}
}


















void FGDKTargetSettingsDetails::OnClickUpdateFromPartnerCenter( UGDKTargetSettings* GDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities )
{
	if (IGDKPlatformEditorModule::Get().IsQueryingPartnerCenter())
	{
		return;
	}
	check(!PartnerCenterProductWindow.IsValid());

	// readability typedefs
	typedef TSharedPtr<IGDKPlatformEditorModule::FPartnerCenterProduct> FProductPtr;
	typedef SListView<FProductPtr> SProductListView;

	// one row in the product selection listbox
	auto OnGenerateRow = []( FProductPtr Product, const TSharedRef<STableViewBase>& OwnerTable )
	{
		const FString DisplayName = Product->FindRef(TEXT("PackageDisplayName"));
		const FString PackageName = Product->FindRef(TEXT("PackageName"));
		return SNew(STableRow<FProductPtr>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(0.4)
			[
				SNew(STextBlock).Text(FText::FromString(DisplayName))
			]

			+SHorizontalBox::Slot()
			.FillWidth(0.6)
			[
				SNew(STextBlock).Text(FText::FromString(PackageName))
			]
		];
	};


	// create the window
	PartnerCenterProductWindow = SNew(SWindow)
		.Title( LOCTEXT("PartnerCenterAssociate","Partner Center Association"))
		.ClientSize(FVector2D(500,600))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule( ESizingRule::UserSized )
		;
	PartnerCenterProductWindow->GetOnWindowClosedEvent().AddLambda( [this](const TSharedRef<SWindow>&)
	{
		IGDKPlatformEditorModule::Get().CancelQueryPartnerCenter();
		PartnerCenterProductWindow.Reset();

	});

	// create the main window contents
	TSharedPtr<SProductListView> ListView;
	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;

	PartnerCenterProductWindow->SetContent(
		SNew(SBorder)
		.Padding(FMargin(8.0f, 8.0f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0)
			[
				SAssignNew(WidgetSwitcher, SWidgetSwitcher)
		
				// progress spinner
				+SWidgetSwitcher::Slot()
				[
					SNew(SBorder)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SThrobber)
					]
				]

				// product selection listbox
				+SWidgetSwitcher::Slot()
				[
					SAssignNew(ListView, SProductListView)
					.ListItemsSource( IGDKPlatformEditorModule::Get().GetPartnerCenterProducts() )
					.SelectionMode(ESelectionMode::SingleToggle)
					.OnGenerateRow_Lambda(OnGenerateRow)
				]			
			]

			// button area
			+SVerticalBox::Slot()
			.Padding(8,4)
			.AutoHeight()
			[
				// ok button
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(5)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("OkButtonLabel", "Ok"))
					.IsEnabled_Lambda( [ListView]() { return (ListView->GetNumItemsSelected() != 0) && !IGDKPlatformEditorModule::Get().IsQueryingPartnerCenter(); } )
					.OnClicked_Lambda( [=, this]()
					{
						const FProductPtr Product = ListView->GetSelectedItems()[0];
						UpdatePartnerCenterData( GDKTargetSettings, *Product, PropertyUtilities );
						PartnerCenterProductWindow->RequestDestroyWindow();
						PartnerCenterProductWindow.Reset();
						return FReply::Handled();
					})
				]

				// cancel button
				+SHorizontalBox::Slot()
				.Padding(5)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
					.OnClicked_Lambda( [this]()
					{
						PartnerCenterProductWindow->RequestDestroyWindow();
						PartnerCenterProductWindow.Reset();
						return FReply::Handled();
					})
				]
			]
		]
	);

	// show the store association dialog
	IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	TSharedPtr<SWindow> ParentWindow = MainFrame.GetParentWindow();
	if ( ParentWindow.IsValid() )
	{
		FSlateApplication::Get().AddModalWindow(PartnerCenterProductWindow.ToSharedRef(), ParentWindow.ToSharedRef(), true);
		PartnerCenterProductWindow->ShowWindow();
	}
	else
	{
		FSlateApplication::Get().AddWindow(PartnerCenterProductWindow.ToSharedRef());
	}

	// refresh the list of products
	auto OnPartnerCenterQueryComplete =  [this,GDKTargetSettings,ListView,WidgetSwitcher](bool bSuccess)
	{
		// show the list view and refresh it
		WidgetSwitcher->SetActiveWidgetIndex(1);
		ListView->RequestListRefresh();

		if (bSuccess)
		{
			// find the currently selected item
			const FProductPtr* CurrentProduct = IGDKPlatformEditorModule::Get().GetPartnerCenterProducts()->FindByPredicate( [GDKTargetSettings]( const FProductPtr Product )
			{
				const FString* StoreId = Product->Find(TEXT("StoreId"));
				return (StoreId != nullptr) && StoreId->Equals(GDKTargetSettings->StoreId);
			});
			if (CurrentProduct && CurrentProduct->IsValid())
			{
				ListView->SetSelection( *CurrentProduct );
			}
		}
		else
		{
			// close the window
			PartnerCenterProductWindow->RequestDestroyWindow();
			PartnerCenterProductWindow.Reset();
		}
	};

	// show the spinner widget and start the query
	WidgetSwitcher->SetActiveWidgetIndex(0);
	IGDKPlatformEditorModule::Get().PartnerCenterQueryProductsAsync(OnPartnerCenterQueryComplete);
}



void FGDKTargetSettingsDetails::UpdatePartnerCenterData( UGDKTargetSettings* GDKTargetSettings, const IGDKPlatformEditorModule::FPartnerCenterProduct& Product, const TSharedRef<class IPropertyUtilities> PropertyUtilities )
{
	auto UpdateProperty = [this, Product, GDKTargetSettings, PropertyUtilities]( FString& Property, const FString& Key, FName MemberName )
	{
		const FString* Value = Product.Find(Key);
		if (Value && !Value->IsEmpty() && !Value->Equals(Property))
		{
			Property = *Value;
			NotifyPropertyChanged(MemberName, GDKTargetSettings, PropertyUtilities);
		}
	};

	UpdateProperty( GDKTargetSettings->TitleId,                TEXT("TitleId"),                GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, TitleId) );
	UpdateProperty( GDKTargetSettings->MSAAppId,               TEXT("MSAAppId"),               GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, MSAAppId) );
	UpdateProperty( GDKTargetSettings->StoreId,                TEXT("StoreId"),                GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, StoreId) );
	UpdateProperty( GDKTargetSettings->PackageName,            TEXT("PackageName"),            GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, PackageName) );
	UpdateProperty( GDKTargetSettings->PrimaryServiceConfigId, TEXT("PrimaryServiceConfigId"), GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, PrimaryServiceConfigId) );
	UpdateProperty( GDKTargetSettings->PublisherName,          TEXT("PublisherName"),          GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, PublisherName) );
	UpdateProperty( GDKTargetSettings->DefaultDisplayName,     TEXT("PackageDisplayName"),     GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, DefaultDisplayName) );
	UpdateProperty( GDKTargetSettings->ProductId,              TEXT("ProductId"),              GET_MEMBER_NAME_CHECKED(UGDKTargetSettings, ProductId) );
	
	PropertyUtilities->ForceRefresh();
}








FString FGDKCultureResourceDetails::GetPickerPath()
{
	return FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
}

bool FGDKCultureResourceDetails::HandlePostExternalIconCopy(const FString& InChosenImage)
{
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InChosenImage));
	return true;
}

TSharedPtr<IPropertyHandle> FGDKCultureResourceDetails::GetDLCNameProperty( TSharedPtr<IPropertyHandle> InStructPropertyHandle ) const
{
	TSharedPtr<IPropertyHandle> ParentPropertyHandle = InStructPropertyHandle->GetParentHandle();

	// the parent of the struct has a field called 'DLCName' (such as FGDKDLCConfiguration::DefaultStringResources)
	TSharedPtr<IPropertyHandle> ChildHandle = ParentPropertyHandle->GetChildHandle("DLCName", false);
	if (ChildHandle.IsValid())
	{
		return ChildHandle;
	}

	// the great-grandparent of the struct has a field called 'DLCName' (such as FGDKDLCConfiguration::PerCultureResources[N])
	ParentPropertyHandle = ParentPropertyHandle->GetParentHandle();
	ParentPropertyHandle = ParentPropertyHandle.IsValid() ? ParentPropertyHandle->GetParentHandle() : nullptr;
	if (ParentPropertyHandle.IsValid())
	{
		ChildHandle = ParentPropertyHandle->GetChildHandle("DLCName", false);
		if (ChildHandle.IsValid())
		{
			return ChildHandle;
		}
	}

	return nullptr;
}

void FGDKCultureResourceDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// add all the child properties (ApplicationDisplayName & ApplicationDescription)
	uint32 NumChildren;
	if (InStructPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumChildren; Index++)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = InStructPropertyHandle->GetChildHandle(Index);
			ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
		}
	}

	// add the button to bring up the image picker dialog
	TSharedPtr<IPropertyHandle> ParentPropertyHandle = InStructPropertyHandle->GetParentHandle();
	TSharedPtr<IPropertyHandle> DLCNameHandle = GetDLCNameProperty(InStructPropertyHandle);
	TSharedPtr<IPropertyHandle> StageIdHandle = ParentPropertyHandle->GetChildHandle("StageId", false);
	FText PickImgLabel = StageIdHandle.IsValid() ? LOCTEXT("PickLocImgsLabel", "Application Images") : LOCTEXT("PickDefImgsLabel", "Application Images");
	FText PickImgToolTip = StageIdHandle.IsValid() ? LOCTEXT("PickLocImgsTip", "Configure the language-specific images for the package") : LOCTEXT("PickDefImgsTip", "Configure the default images for the package");

	ChildBuilder.AddCustomRow(FText::GetEmpty())
		.NameWidget
		[
			SNew(STextBlock)
				.Text(PickImgLabel)
				.Font(StructCustomizationUtils.GetRegularFont())
				.ToolTipText(PickImgToolTip)
		]
		.ValueWidget
		[
			SNew(SButton)
			.Text(LOCTEXT("PickAppImgBtnLabel","Configure..."))
			.ToolTipText(PickImgToolTip)
			.ButtonStyle(FAppStyle::Get(), "Button")
			.OnClicked_Lambda([this, InStructPropertyHandle]()
			{
				OnClickConfigureImages(InStructPropertyHandle);
				return FReply::Handled();
			})
			.IsEnabled_Lambda([=]()
			{
				// if these images belong to DLC, cannot click until the DLC name is selected
				FString DLCName;
				if (DLCNameHandle.IsValid() && (DLCNameHandle->GetValue(DLCName) == FPropertyAccess::Result::Success) && DLCName.IsEmpty())
				{
					return false;
				}

				// default culture always clickable
				if (!StageIdHandle.IsValid())
				{
					return true;
				}

				// otherwise, we must have stage id before this can be clicked
				FString StageId;
				return (StageIdHandle->GetValue(StageId) == FPropertyAccess::Result::Success) && !StageId.IsEmpty(); 
			})
		];
}

void FGDKCultureResourceDetails::OnClickConfigureImages(TSharedRef<IPropertyHandle> InStructPropertyHandle)
{
	// Look up the selected target settings object & find the platform name
	const TCHAR* PlatformName = nullptr;
	UGDKTargetSettings* TargetSettings = nullptr;
	TArray<UObject*> OuterObjects;
	InStructPropertyHandle->GetOuterObjects(OuterObjects);
	for (UObject* OuterObject : OuterObjects)
	{
		TargetSettings = Cast<UGDKTargetSettings>(OuterObject);
		if (TargetSettings != nullptr)
		{
			PlatformName = TargetSettings->GetPlatformName();
			break;
		}
	}
	if (PlatformName == nullptr)
	{
		return;
	}

	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(FName(PlatformName));
	bool bPlatformExtension = PlatformInfo.bIsConfidential;
	FString IniPlatformName = PlatformInfo.IniPlatformName.ToString();

	// look up the Culture Id
	TSharedPtr<IPropertyHandle> ParentPropertyHandle = InStructPropertyHandle->GetParentHandle();
	TSharedPtr<IPropertyHandle> StageIdHandle = ParentPropertyHandle->GetChildHandle("StageId", false);

	FString OwnerName;
	FString CultureId;
	bool bIsDefaultCulture = true;
	if (StageIdHandle.IsValid())
	{
		FString StageId;
		if ((StageIdHandle->GetValue(StageId) == FPropertyAccess::Result::Success) && !StageId.IsEmpty())
		{
			// look up the CultureId for the given StageId
			const FString* CultureIdPtr = TargetSettings->StageIdOverrides.Find(StageId);
			CultureId = CultureIdPtr ? *CultureIdPtr : StageId;
			bIsDefaultCulture = false;
		}
		else
		{
			// If a stage id isn't set we can't display images
			return;
		}
	}


	// lookup DLC
	TSharedPtr<IPlugin> DLCPlugin;
	TSharedPtr<IPropertyHandle> DLCNameHandle = GetDLCNameProperty(InStructPropertyHandle);
	if (DLCNameHandle.IsValid())
	{
		FString DLCName;
		if ((DLCNameHandle->GetValue(DLCName) == FPropertyAccess::Result::Success) && !DLCName.IsEmpty())
		{
			DLCPlugin = IPluginManager::Get().FindPlugin(DLCName);
		}

		// If a DLC name isn't set or the plugin isn't known, we can't display images
		if (!DLCPlugin.IsValid())
		{
			UE_LOGF(LogTemp, Error, "Plugin not found or not specified. %ls", *DLCName);
			return;
		}
	}

	// look up the base directory for the MSGamingSupport plugin to get the default images
	FString MSGamingSupportResourcePath = FPaths::EngineDir() / GDKTargetSettingsDetailsConstants::EngineResourcePath;
	TSharedPtr<IPlugin> MSGamingSupportPlugin = IPluginManager::Get().FindPlugin(TEXT("MSGamingSupport"));
	if (MSGamingSupportPlugin.IsValid())
	{
		FString PluginBaseDir = FPaths::GetPath(MSGamingSupportPlugin->GetDescriptorFileName());
		MSGamingSupportResourcePath = PluginBaseDir / GDKTargetSettingsDetailsConstants::EngineResourcePath;
	}

	// Setup default image path, use the engine copy if we're already dealing with the default culture, otherwise use the default culture
	FString DefaultResourcePath = "";
	if (bIsDefaultCulture)
	{
		if (bPlatformExtension)
		{
			DefaultResourcePath = FPaths::EnginePlatformExtensionDir(PlatformName) / GDKTargetSettingsDetailsConstants::EngineResourcePath;
		}
		else
		{
			DefaultResourcePath = MSGamingSupportResourcePath;
		}
	}
	else
	{
		if (bPlatformExtension)
		{
			DefaultResourcePath = FPaths::ProjectPlatformExtensionDir(PlatformName) / GDKTargetSettingsDetailsConstants::BuildResourcePath;
		}
		else
		{
			DefaultResourcePath = FPaths::ProjectDir() / TEXT("Build") / IniPlatformName / TEXT("MSGaming/Resources");
		}
	}
	DefaultResourcePath = FPaths::ConvertRelativePathToFull(DefaultResourcePath);

	// The target path is always setup to the culture of the control or the root of the resource path if it's the default culture control
	FString TargetResourcePath = "";
	if (DLCPlugin.IsValid())
	{
		if (bPlatformExtension)
		{
			TargetResourcePath = DLCPlugin->GetBaseDir() / TEXT("Platforms") / PlatformName / GDKTargetSettingsDetailsConstants::BuildResourcePath;
		}
		else
		{
			TargetResourcePath = DLCPlugin->GetBaseDir() / TEXT("Build") / IniPlatformName / TEXT("MSGaming/Resources");
		}
	}
	else
	{
		if (bPlatformExtension)
		{
			TargetResourcePath = FPaths::ProjectPlatformExtensionDir(PlatformName) / GDKTargetSettingsDetailsConstants::BuildResourcePath;
		}
		else
		{
			TargetResourcePath = FPaths::ProjectDir() / TEXT("Build") / IniPlatformName / TEXT("MSGaming/Resources");
		}

	}
	if (!bIsDefaultCulture)
	{
		TargetResourcePath /= CultureId;
	}
	TargetResourcePath = FPaths::ConvertRelativePathToFull(TargetResourcePath);



	// prepare window title, showing the selected culture id if appropriate
	FText WindowTitle = LOCTEXT("PickAppImgDefault", "Default Packaging Images");
	if (!bIsDefaultCulture)
	{
		const FText CultureIdText = FText::FromString(CultureId);
		WindowTitle = FText::Format(LOCTEXT("PickAppImgLocale", "Packaging Images For Locale \"{0}\""), CultureIdText);
	}

	// create the window
	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(600,800))
		.SizingRule(ESizingRule::UserSized);

	TSharedPtr<SScrollBox> ImagesBox;
	Window->SetContent(
		SNew(SBorder)
		.Padding(FMargin(8.0f, 8.0f))
		[
			// product selection listbox
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0)
			.Padding(5, 5)
			[
				SAssignNew(ImagesBox, SScrollBox)
			]

			// button area
			+SVerticalBox::Slot()
			.Padding(8,4)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// open resource images folder
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				[
					SNew(SHyperlink)
					.Text(LOCTEXT("OpenFolder", "Open Folder"))
					.ToolTipText(FText::FromString(TargetResourcePath))
					.OnNavigate_Lambda( [TargetResourcePath]() { IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*TargetResourcePath); FPlatformProcess::ExploreFolder(*TargetResourcePath); })
				]

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SNew(SButton)
					.Text(LOCTEXT("Close", "Close"))
					.OnClicked_Lambda( [=]()
					{
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]
		]
	);


	// Application small logo widget
	const FText AppSmLogoDesc(LOCTEXT("AppSmLogoLabel", "Application 44x44 Small Logo"));
	const FString AppSmLogo_TargetImagePath(TargetResourcePath / "SmallLogo.png");
	const FString AppSmLogo_DefaultImagePath(DefaultResourcePath / "SmallLogo.png");
	AddImagePickerRow(ImagesBox, AppSmLogoDesc, AppSmLogo_DefaultImagePath, AppSmLogo_TargetImagePath, 44, 44);

	// Store logo widget
	const FText StoreLogoDesc(LOCTEXT("StoreLogoLabel", "Store 100x100 Logo"));
	const FString StoreLogo_TargetImagePath(TargetResourcePath / "StoreLogo.png");
	const FString StoreLogo_DefaultImagePath(DefaultResourcePath / "StoreLogo.png");
	AddImagePickerRow(ImagesBox, StoreLogoDesc, StoreLogo_DefaultImagePath, StoreLogo_TargetImagePath, 100, 100);

 	// Application logo widget
 	const FText AppLogoDesc(LOCTEXT("AppLogoLabel", "Application 150x150 Logo"));
 	const FString AppLogo_TargetImagePath(TargetResourcePath / "Logo.png");
 	const FString AppLogo_DefaultImagePath(DefaultResourcePath / "Logo.png");
	AddImagePickerRow(ImagesBox, AppLogoDesc, AppLogo_DefaultImagePath, AppLogo_TargetImagePath, 150, 150);

	// Application large logo widget
 	const FText AppLgLogoDesc(LOCTEXT("AppLargeLogoLabel", "Application 480x480 Large Logo"));
 	const FString AppLgLogo_TargetImagePath(TargetResourcePath / "Square480x480Logo.png");
 	const FString AppLgLogo_DefaultImagePath(DefaultResourcePath / "Square480x480Logo.png");
	AddImagePickerRow(ImagesBox, AppLgLogoDesc, AppLgLogo_DefaultImagePath, AppLgLogo_TargetImagePath, 480, 480);

 	// Application splash screen widget
 	const FText AppSplashDesc(LOCTEXT("AppSplashLabel", "Application 1920x1080 Splash Screen"));
 	const FString AppSplash_TargetImagePath(TargetResourcePath / "SplashScreen.png");
 	const FString AppSplash_DefaultImagePath(DefaultResourcePath / "SplashScreen.png");
	AddImagePickerRow(ImagesBox, AppSplashDesc, AppSplash_DefaultImagePath, AppSplash_TargetImagePath, 1920, 1080);


	// show the window
	IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	TSharedPtr<SWindow> ParentWindow = MainFrame.GetParentWindow();
	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}
}




void FGDKCultureResourceDetails::AddImagePickerRow(TSharedPtr<class SScrollBox> ImagesBox, const FText& ImageDescription, const FString& DefaultImagePath, const FString& TargetImagePath, int32 ImageWidth, int32 ImageHeight)
{
	TArray<FString> ImageExtensions;
	ImageExtensions.Add(TEXT("png"));

	ImagesBox->AddSlot()
		[
			SNew(STextBlock)
			.Text(ImageDescription)
		];

	ImagesBox->AddSlot()
	.Padding(4,0,4,16)
	[
		SNew(SExternalImageReference, DefaultImagePath, TargetImagePath)
		.FileDescription(ImageDescription)
		.RequiredSize(FIntPoint(ImageWidth, ImageHeight))
		.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FGDKCultureResourceDetails::GetPickerPath))
		.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FGDKCultureResourceDetails::HandlePostExternalIconCopy))
		.DeleteTargetWhenDefaultChosen(true)
		.FileExtensions(ImageExtensions)
		.DeletePreviousTargetWhenExtensionChanges(true)
	];
}

#undef LOCTEXT_NAMESPACE
