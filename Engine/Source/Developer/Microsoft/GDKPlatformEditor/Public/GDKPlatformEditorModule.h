// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Containers/Map.h"

class IGDKPlatformEditorModule : public IModuleInterface
{
public:	
	static inline IGDKPlatformEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IGDKPlatformEditorModule>("GDKPlatformEditor");
	}

	typedef TMap<FString,FString> FPartnerCenterProduct;

	virtual void PartnerCenterQueryProductsAsync( TFunction<void(bool)> OnComplete ) = 0;
	virtual const TArray<TSharedPtr<FPartnerCenterProduct>>* GetPartnerCenterProducts() = 0;

	virtual bool IsQueryingPartnerCenter() const = 0;
	virtual void CancelQueryPartnerCenter() = 0;
};
