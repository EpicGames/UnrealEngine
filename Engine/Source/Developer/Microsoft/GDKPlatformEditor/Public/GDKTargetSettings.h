// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GDKTargetSettings.h: Declares the UGDKTargetSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "GDKTargetSettings.generated.h"

#define UE_API GDKPLATFORMEDITOR_API

UENUM()
enum class EGDKForegroundColor : uint8
{
	/** Use light colored text to contrast against a dark background. */
	Light,

	/** Use dark colored text to contrast against a light background. */
	Dark,
};


USTRUCT()
struct FGDKPackageStringResources
{
	GENERATED_USTRUCT_BODY()

	/**
	* Friendly application display name. This value will be displayed on game tiles in the dashboard.
	*/
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FString ApplicationDisplayName;

	/**
	* Friendly application description. This value will be displayed on game tiles in the game collection.
	*/
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FString ApplicationDescription;
};

USTRUCT()
struct FGDKLocalizedPackageResources
{
	GENERATED_USTRUCT_BODY()

	/**
	* Which CultureToStage entry these resources should apply to. See "Languages To Package" in Packaging settings
	*/
	UPROPERTY(EditAnywhere, Config, Category=Content, Meta=(GetOptions="GetKnownCultureStageIds", DisplayName="Language"))
	FString StageId;

	/**
	* The package localized strings.
	*/
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FGDKPackageStringResources CultureStringResources;
};





UENUM()
enum class EGDKCustomChunkMappingType : uint8
{
	Main,
	Optional,
};

USTRUCT()
struct FGDKCustomChunkMapping
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Config, Category=Packaging, Meta=(ClampMin=0))
	int32 ChunkId = 0;

	// relative content path: e.g. shootergame/content/movies/\*  or  shootergame/content/movies/movie1.mp4
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	FString Pattern;

	// not currently wired up, so keeping at default
	UPROPERTY()
	EGDKCustomChunkMappingType Type = EGDKCustomChunkMappingType::Main;
};




USTRUCT()
struct FGDKIntelligentDeliveryChunk
{
	GENERATED_USTRUCT_BODY()

	// Unreal chunk identifier
	UPROPERTY(EditAnywhere, Config, Category=Packaging, Meta=(ClampMin=0))
	int32 ChunkId = 0;

	// List of tags, separated by semicolons, (using # to define groups if Features are also used). Does not apply when 'required for launch' is set. See the GDK documentation for details
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	FString Tags;

	// List of languages from CultureToStage. See "Languages To Package" in Packaging settings
	UPROPERTY(EditAnywhere, Config, Category=Packaging, Meta=(GetOptions="GetKnownCultureStageIds", DisplayName="Languages"))
	TArray<FString> StageIds;

	// List of devices that this chunk is available for.
	UPROPERTY(EditAnywhere, Config, Category=Packaging, Meta=(GetOptions="GetKnownDeviceTypes"))
	TArray<FString> Devices;

	// Whether this chunk must be installed before the game can be launched.
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	bool bRequiredForLaunch = false;

};

USTRUCT()
struct FGDKLocalizedIntelligentDeliveryFeatureResource
{
	GENERATED_USTRUCT_BODY()

	// Locale from CultureToStage. See "Languages To Package" in Packaging settings
	UPROPERTY(EditAnywhere, Config, Category=Content, Meta=(GetOptions="GetKnownCultureStageIds", DisplayName="Language"))
	FString StageId;

	// Friendly install feature display name. This value will be displayed to the user in the shell
	UPROPERTY(EditAnywhere, Config, Category=Content, AdvancedDisplay)
	FString DisplayName;
};


USTRUCT()
struct FGDKIntelligentDeliveryFeature
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Config, Category=Feature)
	FString Id;

	// Semicolon-separated list of chunk tags. See the GDK documentation for details
	UPROPERTY(EditAnywhere, Config, Category=Feature)
	FString Tags;

	// Whether this feature is hidden from the user. See the GDK documentation for details. 
	UPROPERTY(EditAnywhere, Config, Category=Feature)
	bool bHidden = true;

	// Friendly install feature display name. This value will be displayed to the user in the shell.
	UPROPERTY(EditAnywhere, Config, Category=Content, Meta=(EditCondition="!bHidden"))
	FString DefaultDisplayName;

	// Optional per-language resources that will be used for this feature
	UPROPERTY(EditAnywhere, Config, Category=Content, DisplayName="Per-Language Resources", Meta=(EditCondition="!bHidden", TitleProperty="StageId"))
	TArray<FGDKLocalizedIntelligentDeliveryFeatureResource> PerCultureResources;
};


USTRUCT()
struct FGDKIntelligentDeliveryRecipe
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Config, Category=Recipe)
	FString Id;

	// List of features to include, separated by semicolons. See the GDK documentation for details
	UPROPERTY(EditAnywhere, Config, Category=Recipe, Meta=(GetOptions="GetKnownIntelligentDeliveryFeatures"))
	TArray<FString> IncludedFeatures;

	// List of devices that this recipe is available for.
	UPROPERTY(EditAnywhere, Config, Category=Recipe, Meta=(GetOptions="GetKnownDeviceTypes"))
	TArray<FString> Devices;

	// Premium product recipe. Only installed if the user owns one of these products. See the GDK documentation for details
	UPROPERTY(EditAnywhere, Config, Category=Recipe)
	TArray<FString> StoreIds;
};

USTRUCT()
struct FGDKDLCPackageStringResources
{
	GENERATED_USTRUCT_BODY();

	// Locale from CultureToStage. See "Languages To Package" in Packaging settings
	UPROPERTY(EditAnywhere, Config, Category=Content, Meta=(GetOptions="GetKnownCultureStageIds", DisplayName="Language"))
	FString StageId;

	// The localized strings
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FGDKPackageStringResources CultureStringResources;
};


USTRUCT()
struct FGDKDLCPackage
{
	GENERATED_USTRUCT_BODY()

	/* Name of this DLC. Used by the various DLC support APIs */
	UPROPERTY(EditAnywhere, Config, Category=DLC, DisplayName="DLC Name")
	FString DLCName;

	/* Store Id. Must match Partner Center configuration. */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter)
	FString StoreId;

	/* Name of the product in Partner Center. Must match Partner Center configuration */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Product Name")
	FString DefaultDisplayName;

	/* Package/Identity/Name. Must match Partner Center configuration */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Package/Identity/Name")
	FString PackageName;

	/* Package version in the form 0.0.0.0. Leave blank to inherit the main package version */
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	FString PackageVersion;


	/* The package default strings. */
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FGDKPackageStringResources DefaultStringResources;

	/* The per culture images, icons, and strings describing a package. */
	UPROPERTY(EditAnywhere, Config, Category=Content, DisplayName="Per-Language Resources", Meta=(TitleProperty="StageId"))
	TArray<FGDKDLCPackageStringResources> PerCultureResources;
};

USTRUCT()
struct FGDKDLCChunk
{
	GENERATED_USTRUCT_BODY()

	/* Name of the DLC to associate with this chunk */
	UPROPERTY(EditAnywhere, Config, Category=DLC, DisplayName="DLC Name", Meta=(GetOptions="GetKnownDLCNames"))
	FString DLCName;

	/* Unreal chunk identifier that will be moved into this DLC */
	UPROPERTY(EditAnywhere, Config, Category=DLC, Meta=(ClampMin=1))
	int32 ChunkId = 1;
};

USTRUCT()
struct FGDKDLCPlugins
{
	GENERATED_USTRUCT_BODY()

	/* Name of the DLC to associate with this plugin */
	UPROPERTY(EditAnywhere, Config, Category=DLC, DisplayName="DLC Name", Meta=(GetOptions="GetKnownDLCNames"))
	FString DLCName;

	/* Name of the explicity-loaded content plugin this DLC applies to */
	UPROPERTY(EditAnywhere, Config, Category=DLC, Meta=(GetOptions="GetKnownDLCPlugins", DisplayName="Plugin"))
	FString Plugin;
};


USTRUCT()
struct FGDKUriActivationProtocol
{
	GENERATED_USTRUCT_BODY()

	/* e.g. MyGameUGC
	 * Protocol that is registered with the OS. This will be what is launched by the user via MyGameUGC://<payload>
	 * note that this will be system-wide, so should be clearly unique for your title.
	 */
	UPROPERTY(EditAnywhere, Config, Category=ActivationProtocol, Meta=(DisplayName="OS-Registered Protocol", MaxLength=39))
	FString RegisteredProtocol;

	/* e.g. ugc://
	 * Prefix for how this protocol's payload will be represented in FCoreDelegates::OnActivatedByProtocol as ugc://<payload> 
	 * note that this is only visible to your game, so can be a generic name.
	 * this can be combined with other <platform>.*UriPrefix CVars to craft platform-agnostic OnActivatedByProtocol handler code.
	 */
	UPROPERTY(EditAnywhere, Config, Category=ActivationProtocol)
	FString InGameUriPrefix;
};

UENUM()
enum class EGDKCustomInstallActionType : uint8
{
	Invalid UMETA(Hidden, DisplayName="..."),
	Install,
	Repair,
	Uninstall UMETA(ToolTip="Note: there is a bug in Windows 10 that prevents the uninstall action from being run. It is fixed in Windows 11"),
};

USTRUCT()
struct FGDKCustomInstallCommand
{
	GENERATED_USTRUCT_BODY()

	/* .exe or .msi file to run */
	UPROPERTY(EditAnywhere, Config, Category=Action, Meta=(FilePathFilter="Commands (*.exe;*.msi)|*.exe;*.msi|All Files (*.*)|*.*"))
	FFilePath Command;

	/* command line arguments to pass to the given command. For .msi files this can be blank */
	UPROPERTY(EditAnywhere, Config, Category=Action)
	FString Arguments;
};

USTRUCT()
struct FGDKCustomInstallAction
{
	GENERATED_USTRUCT_BODY()

	/* name of the item to be installed/repaired/uninstalled. SourceFolder will be staged into a subfolder with this name */
	UPROPERTY(EditAnywhere, Config, Category=Action)
	FString Name;

	/* Source folder for all the custom install commands. The contents of this directory will be staged automatically */
	UPROPERTY(EditAnywhere, Config, Category=Actions, Meta=(RelativePath))
	FDirectoryPath SourceFolder;

	/* Relative destination folder when staging, relative to Custom Install Staging Root. Leave blank to stage the custom install action into a subfolder  */
	UPROPERTY(EditAnywhere, Config, Category=Action)
	FString TargetFolder;

	/* The commands to run for each step of the installation */
	UPROPERTY(EditAnywhere, Config, Category=Action)
	TMap<EGDKCustomInstallActionType,FGDKCustomInstallCommand> Commands;
};



/**
 * Implements the settings for the GDK target platform.
  * Note: Properties must use Config not GlobalConfig because the properties need to be serialized into the specialization's own ini file.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig, Abstract)
class UGDKTargetSettings
	: public UObject
{
public:

	GENERATED_UCLASS_BODY()

	virtual const TCHAR* GetPlatformName() const PURE_VIRTUAL(GetPlatformName, return nullptr;);

	/* Inform the OS of what color scheme best contrasts with your application's tile images. */
	UPROPERTY(EditAnywhere, Config, Category=Content, Meta=(DisplayName="Application Foreground Text"))
	EGDKForegroundColor ApplicationForegroundColor;

	/* Inform the OS what background color to use for your application tiles. */
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FColor ApplicationBackgroundColor;

	/* The package default strings. */
	UPROPERTY(EditAnywhere, Config, Category=Content)
	FGDKPackageStringResources CultureStringResources;

	/* The per culture images, icons, and strings describing a package. */
	UPROPERTY(EditAnywhere, Config, Category=Content, DisplayName="Per-Language Resources", meta = (TitleProperty="StageId"))
	TArray<FGDKLocalizedPackageResources> PerCultureResources;

	/* Mapping from "Languages To Package" in Packaging settings, to "Console-supported languages" in the GDK documentation. Leave empty to keep 1:1 mapping */
	UPROPERTY(EditAnywhere, Config, Category=Content, AdvancedDisplay, Meta=(GetKeyOptions="GetKnownCultureStageIds", DisplayName="Language/Locale Overrides"))
	TMap<FString,FString> StageIdOverrides;


	/* Whether to use the Simplified User Model where the OS will maintain a primary user for the title and the title will be terminated if this user signs out. Other secondary users are free to sign in and out. See the GDK documention for details. */
	UPROPERTY(EditAnywhere, Config, Category=Features)
	bool bUseSimplifiedUserModel = false;

	/* List of custom protocol schemes to register (March 2023 GDK onwards) */
	UPROPERTY(EditAnywhere, Config, Category=Features, Meta=(TitleProperty="RegisteredProtocol", ForMSGameStore))
	TArray<FGDKUriActivationProtocol> UriActivationProtocols;



	/* Name of the product in Partner Center. Must match Partner Center configuration */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter)
	FString DefaultDisplayName;

	/* Package/Identity/Name. Must match Partner Center configuration */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Package/Identity/Name")
	FString PackageName;

	/* Package/Identity/Publisher. Must match Partner Center configuration */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Package/Identity/Publisher")
	FString PublisherName;

	/* Package/Properties/PublisherDisplayName. Must match Partner Center configuration */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Package/Properties/PublisherDisplayName")
	FString PublisherDisplayName;

	/* Store Id. Must match Partner Center configuration. */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Store ID")
	FString StoreId;

	/* Xbox Live Title Id. Get this value from your Partner Center configuration. This value is required for title submission. */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Xbox Title ID")
	FString TitleId;

	/* Xbox Live Service Config Id (SCID). Get this value from your Partner Center configuration. This value is required for title submission. */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Xbox Services Configuration ID (SCID)")
	FString PrimaryServiceConfigId;

	/* MSA App Id. Get this value from your Partner Center configuration. This value is required for title submission. */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="MSA app Id")
	FString MSAAppId;

	/* Legacy Product GUID. Must match Partner Center configuration. Ignored if StoreId is specified */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, DisplayName="Legacy Product ID", Meta=(EditCondition="CanEditProductId()"))
	FString ProductId;

	UFUNCTION()
	UE_API bool CanEditProductId() const;



	/* Package version in the form 0.0.0.0 */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter)
	FString PackageVersion;

	/* Use Include the current engine changelist in the package version number. */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, AdvancedDisplay)
	bool bIncludeEngineVersionInPackageVersion;




	/* Define custom chunk mapping */
	UPROPERTY(EditAnywhere, Config, Category="Chunk Install", AdvancedDisplay, Meta = (TitleProperty="Pattern"))
	TArray<FGDKCustomChunkMapping> CustomChunkMapping;

	/* Whether to use simple On-Demand intelligent install chunks, or new Features and Recipes for defining chunks and installations */
	UPROPERTY(EditAnywhere, Config, Category="Chunk Install")
	bool bUseFeaturesAndRecipes;

	/* List of chunks. See the GDK documentation for details */
	UPROPERTY(EditAnywhere, Config, Category="Chunk Install", Meta = (TitleProperty="ChunkId"))
	TArray<FGDKIntelligentDeliveryChunk> IntelligentDeliveryChunks;

	/* List of features. See the GDK documentation for details */
	UPROPERTY(EditAnywhere, Config, Category="Chunk Install", Meta = (EditCondition="bUseFeaturesAndRecipes", TitleProperty="Id"))
	TArray<FGDKIntelligentDeliveryFeature> IntelligentDeliveryFeatures;

	/* List of recipes. See the GDK documentation for details */
	UPROPERTY(EditAnywhere, Config, Category="Chunk Install", Meta = (EditCondition="bUseFeaturesAndRecipes", TitleProperty="Id"))
	TArray<FGDKIntelligentDeliveryRecipe> IntelligentDeliveryRecipes;


	// UI helper functions to get the list of known devices, used by FGDKIntelligentDeliveryRecipe & FGDKIntelligentDeliveryChunk
	UFUNCTION()
	UE_API virtual TArray<FString> GetKnownDeviceTypes();

	// UI helper function to get the list of known features, used by FGDKIntelligentDeliveryRecipe
	UFUNCTION()
	UE_API TArray<FString> GetKnownIntelligentDeliveryFeatures();

	// UI helper function to get the list of known culture StageIDs, used by FGDKIntelligentDeliveryChunk
	UFUNCTION()
	UE_API TArray<FString> GetKnownCultureStageIds();

	// UI helper function to get the list of DLC packages
	UFUNCTION()
	UE_API TArray<FString> GetKnownDLCNames();

	// UI helper function to get the list of known DLC plugins
	UFUNCTION()
	UE_API TArray<FString> GetKnownDLCPlugins();
	

	/* Configuration for DLC packages. */
	UPROPERTY(EditAnywhere, Config, Category=DLC, Meta=(TitleProperty="DLCName", DisplayName="DLC Packages (EXPERIMENTAL)"))
	TArray<FGDKDLCPackage> DLCPackages;

	/* When enabled, the DLC packages listed above will be generated automatically using `DLC Chunks` during normal packaging. This is not compatible with the "DLC Plugins" mechanism or packaging a single plugin in UAT via -dlcname= */
	UPROPERTY(EditAnywhere, Config, Category=DLC, Meta=(DisplayName="Automatically Create DLC Packages (EXPERIMENTAL)"))
	bool bAutoCreateDLCPackages = false;

	/* Chunks to pull out of the main package to include in DLC instead */
	UPROPERTY(EditAnywhere, Config, Category=DLC, Meta=(TitleProperty="ChunkId", DisplayName="DLC Chunks (EXPERIMENTAL)", EditCondition="bAutoCreateDLCPackages", EditConditionHides="true"))
	TArray<FGDKDLCChunk> DLCChunks;

	/* Configuration for DLC plugins */
	UPROPERTY(EditAnywhere, Config, Category=DLC, Meta=(TitleProperty="DLCName", DisplayName="DLC Plugins (EXPERIMENTAL)", EditCondition="!bAutoCreateDLCPackages", EditConditionHides="true"))
	TArray<FGDKDLCPlugins> DLCPlugins;
};

#undef UE_API
