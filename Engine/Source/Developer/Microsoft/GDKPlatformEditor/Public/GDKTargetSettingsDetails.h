// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PropertyHandle.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "GDKPlatformEditorModule.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWindow.h"

#define UE_API GDKPLATFORMEDITOR_API

/**
* Detail customization for GDK target settings panel
*/
class FGDKTargetSettingsDetails : public IDetailCustomization
{
public:
	UE_API FGDKTargetSettingsDetails(const TCHAR* InPlatformName);

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	UE_API void NotifyPropertyChanged(FProperty* Property, UObject* OwnerObject, const TSharedRef<class IPropertyUtilities> PropertyUtilities) const;
	UE_API void NotifyPropertyChanged(FName PropertyName, UObject* OwnerObject, const TSharedRef<class IPropertyUtilities> PropertyUtilities) const;

	virtual void AddAdditionalPlatformItems( TSharedPtr<class SHorizontalBox> PartnerCenterBox, TWeakObjectPtr<class UGDKTargetSettings> GDKTargetSettings ) {};
	virtual void AddAdditionalGettingStartedItems( TSharedPtr<class SHorizontalBox> GettingStartedBox, TWeakObjectPtr<class UGDKTargetSettings> GDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities ) {};

	UE_API bool IsGDKConfigured(class UGDKTargetSettings* GDKTargetSettings) const;

	FString PlatformName;

private:
	UE_API void AddGettingStartedItems(TWeakObjectPtr<class UGDKTargetSettings> GDKTargetSettings, TSharedPtr<class SHorizontalBox> PartnerCenterBox, const TSharedRef<class IPropertyUtilities> PropertyUtilities);
	UE_API void OnClickUpdateFromPartnerCenter( UGDKTargetSettings* GDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities );
	UE_API void UpdatePartnerCenterData( class UGDKTargetSettings* GDKTargetSettings, const IGDKPlatformEditorModule::FPartnerCenterProduct& Product, const TSharedRef<class IPropertyUtilities> PropertyUtilities );

	TSharedPtr<SWindow> PartnerCenterProductWindow;
};

class FGDKCultureResourceDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FGDKCultureResourceDetails);
	}

	/** Overridden to do nothing */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

	/** Overridden display image customization options for packaging */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	/** Displays a dialog to set up the images for the given culture */
	void OnClickConfigureImages(TSharedRef<IPropertyHandle> InStructPropertyHandle);

	/** Creates a packaging image picker */
	void AddImagePickerRow(TSharedPtr<class SScrollBox> ImagesBox, const FText& ImageDescription, const FString& DefaultImagePath, const FString& TargetImagePath, int32 ImageWidth, int32 ImageHeight);

private:
	TSharedPtr<IPropertyHandle> GetDLCNameProperty( TSharedPtr<IPropertyHandle> InStructPropertyHandle ) const;
	/** Delegate handler to get the path to start picking from */
	FString GetPickerPath();

	/** Delegate handler to set the path to start picking from */
	bool HandlePostExternalIconCopy(const FString& InChosenImage);
};

#undef UE_API
